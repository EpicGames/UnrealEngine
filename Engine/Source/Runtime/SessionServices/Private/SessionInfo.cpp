// Copyright Epic Games, Inc. All Rights Reserved.

#include "SessionInfo.h"
#include "EngineServiceMessages.h"
#include "SessionServiceMessages.h"
#include "SessionInstanceInfo.h"


/* FSessionInfo structors
 *****************************************************************************/

FSessionInfo::FSessionInfo(const FGuid& InSessionId, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus)
	: MessageBusPtr(InMessageBus)
	, SessionId(InSessionId)
{ }


/* FSessionInfo interface
 *****************************************************************************/

void FSessionInfo::UpdateFromMessage(const FEngineServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.SessionId != SessionId)
	{
		return;
	}

	// update instance
	// @todo gmp: reconsider merging EngineService and SessionService
	/*TSharedPtr<FSessionInstanceInfo> Instance = Instances.FindRef(Context->GetSender());

	if (Instance.IsValid())
	{
		Instance->UpdateFromMessage(Message, Context);
	}*/
	for (TMap<FMessageAddress, TSharedPtr<FSessionInstanceInfo> >::TIterator It(Instances); It; ++It)
	{
		if (It.Value()->GetInstanceId() == Message.InstanceId)
		{
			It.Value()->UpdateFromMessage(Message, Context);
			break;
		}
	}
}


void FSessionInfo::UpdateFromMessage(const FSessionServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.SessionId != SessionId)
	{
		return;
	}

	// update session info
	Standalone = Message.Standalone;
	SessionOwner = Message.SessionOwner;
	SessionName = Message.SessionName;

	// update instance
	TSharedPtr<FSessionInstanceInfo>& Instance = Instances.FindOrAdd(Context->GetSender());

	if (Instance.IsValid())
	{
		Instance->UpdateFromMessage(Message, Context);
	}
	else
	{
		auto MessageBus = MessageBusPtr.Pin();

		if (MessageBus.IsValid())
		{
			Instance = MakeShareable(new FSessionInstanceInfo(Message.InstanceId, AsShared(), MessageBus.ToSharedRef()));
			Instance->OnLogReceived().AddSP(this, &FSessionInfo::HandleLogReceived);
			Instance->UpdateFromMessage(Message, Context);

			InstanceDiscoveredEvent.Broadcast(AsShared(), Instance.ToSharedRef());
		}
	}

	LastUpdateTime = FDateTime::UtcNow();
}


/* ISessionInfo interface
 *****************************************************************************/

void FSessionInfo::GetInstances(TArray<TSharedPtr<ISessionInstanceInfo>>& OutInstances) const
{
	OutInstances.Empty();

	for (TMap<FMessageAddress, TSharedPtr<FSessionInstanceInfo> >::TConstIterator It(Instances); It; ++It)
	{
		OutInstances.Add(It.Value());
	}
}


const FDateTime& FSessionInfo::GetLastUpdateTime() const
{
	return LastUpdateTime;
}


const int32 FSessionInfo::GetNumInstances() const
{
	return Instances.Num();
}


const FGuid& FSessionInfo::GetSessionId() const
{
	return SessionId;
}


const FString& FSessionInfo::GetSessionName() const
{
	return SessionName;
}


const FString& FSessionInfo::GetSessionOwner() const
{
	return SessionOwner;
}


const bool FSessionInfo::IsStandalone() const
{
	return Standalone;
}


void FSessionInfo::Terminate()
{
	for (TMap<FMessageAddress, TSharedPtr<FSessionInstanceInfo> >::TIterator It(Instances); It; ++It)
	{
		It.Value()->Terminate();
	}
}


/* FSessionInfo callbacks
 *****************************************************************************/

void FSessionInfo::HandleLogReceived(const TSharedRef<ISessionInstanceInfo>& Instance, const TSharedRef<FSessionLogMessage>& LogMessage)
{
	LogReceivedEvent.Broadcast(AsShared(), Instance, LogMessage);
}
