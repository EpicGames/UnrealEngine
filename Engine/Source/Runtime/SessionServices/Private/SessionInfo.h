// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "IMessageContext.h"
#include "IMessageBus.h"
#include "ISessionInstanceInfo.h"
#include "ISessionInfo.h"
#include "SessionLogMessage.h"

class FSessionInstanceInfo;
struct FEngineServicePong;
struct FSessionServicePong;

/**
 * Implements a class to maintain all info related to a game session
 */
class FSessionInfo
	: public TSharedFromThis<FSessionInfo>
	, public ISessionInfo
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSessionId The session's identifier.
	 * @param InMessageBus The message bus to use.
	 */
	FSessionInfo(const FGuid& InSessionId, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus);

public:

	/**
	 * Updates this session info with the data in the specified message.
	 *
	 * @param Message The message containing engine information.
	 * @param Context The message context.
	 */
	void UpdateFromMessage(const FEngineServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/**
	 * Updates this session info with the data in the specified message.
	 *
	 * @param Message The message containing session information.
	 * @param Context The message context.
	 */
	void UpdateFromMessage(const FSessionServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

public:	

	//~ ISessionInfo interface

	virtual void GetInstances(TArray<TSharedPtr<ISessionInstanceInfo>>& OutInstances) const override;
	virtual const FDateTime& GetLastUpdateTime() const override;
	virtual const int32 GetNumInstances() const override;
	virtual const FGuid& GetSessionId() const override;
	virtual const FString& GetSessionName() const override;
	virtual const FString& GetSessionOwner() const override;
	virtual const bool IsStandalone() const override;

	DECLARE_DERIVED_EVENT(FSessionInfo, ISessionInfo::FInstanceDiscoveredEvent, FInstanceDiscoveredEvent)
	virtual FInstanceDiscoveredEvent& OnInstanceDiscovered() override
	{
		return InstanceDiscoveredEvent;
	}

	DECLARE_DERIVED_EVENT(FSessionInfo, ISessionInfo::FLogReceivedEvent, FLogReceivedEvent)
	virtual FLogReceivedEvent& OnLogReceived() override
	{
		return LogReceivedEvent;
	}

	virtual void Terminate() override;

private:

	/** Handles received log messages. */
	void HandleLogReceived(const TSharedRef<ISessionInstanceInfo>& Instance, const TSharedRef<FSessionLogMessage>& LogMessage);

private:

	/** Holds the list of engine instances that belong to this session. */
	TMap<FMessageAddress, TSharedPtr<FSessionInstanceInfo>> Instances;

	/** Holds the time at which the last pong was received. */
	FDateTime LastUpdateTime;

	/** Holds a weak pointer to the message bus. */
	TWeakPtr<IMessageBus, ESPMode::ThreadSafe> MessageBusPtr;

	/** Holds the session identifier. */
	FGuid SessionId;

	/** Holds the session name. */
	FString SessionName;

	/** Holds the name of the user who launched the session. */
	FString SessionOwner;

	/** Whether the session is local. */
	bool Standalone;

private:

	/** Holds a delegate to be invoked when a new instance has been discovered. */
	FInstanceDiscoveredEvent InstanceDiscoveredEvent;

	/** Holds a delegate to be invoked when an instance received a log message. */
	FLogReceivedEvent LogReceivedEvent;
};
