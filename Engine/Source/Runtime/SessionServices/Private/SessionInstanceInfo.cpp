// Copyright Epic Games, Inc. All Rights Reserved.

#include "SessionInstanceInfo.h"

#include "EngineServiceMessages.h"
#include "HAL/PlatformProcess.h"
#include "MessageEndpointBuilder.h"
#include "SessionServiceMessages.h"

#include "SessionLogMessage.h"


/* FSessionInstanceInfo structors
 *****************************************************************************/

FSessionInstanceInfo::FSessionInstanceInfo(const FGuid& InInstanceId, const TSharedRef<ISessionInfo>& InOwner, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus)
	: Authorized(false)
	, EngineVersion(0)
	, InstanceId(InInstanceId)
	, Owner(InOwner)
{
	MessageEndpoint = FMessageEndpoint::Builder("FSessionInstanceInfo", InMessageBus)
		.Handling<FSessionServiceLog>(this, &FSessionInstanceInfo::HandleSessionLogMessage);
}

FSessionInstanceInfo::~FSessionInstanceInfo()
{
	FMessageEndpoint::SafeRelease(MessageEndpoint);
}

/* FSessionInstanceInfo interface
 *****************************************************************************/

void FSessionInstanceInfo::UpdateFromMessage(const FEngineServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.InstanceId != InstanceId)
	{
		return;
	}

	CurrentLevel = Message.CurrentLevel;
	EngineAddress = Context->GetSender();
	EngineVersion = Message.EngineVersion;
	HasBegunPlay = Message.HasBegunPlay;
	WorldTimeSeconds = Message.WorldTimeSeconds;
	InstanceType = Message.InstanceType;
}


void FSessionInstanceInfo::UpdateFromMessage(const FSessionServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.InstanceId != InstanceId)
	{
		return;
	}

	if (MessageEndpoint.IsValid() && (ApplicationAddress != Context->GetSender()))
	{
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FSessionServiceLogSubscribe>(), Context->GetSender());
	}

	Authorized = Message.Authorized;
	ApplicationAddress = Context->GetSender();
	BuildDate = Message.BuildDate;
	DeviceName = Message.DeviceName;
	InstanceName = Message.InstanceName;
	PlatformName = Message.PlatformName;

	LastUpdateTime = FDateTime::UtcNow();
}


/* FSessionInstanceInfo interface
 *****************************************************************************/

void FSessionInstanceInfo::ExecuteCommand(const FString& CommandString)
{
	if (MessageEndpoint.IsValid() && EngineAddress.IsValid())
	{
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FEngineServiceExecuteCommand>(CommandString, FPlatformProcess::UserName(false)), EngineAddress);
	}
}


void FSessionInstanceInfo::Terminate()
{
	if (MessageEndpoint.IsValid() && EngineAddress.IsValid())
	{
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FEngineServiceTerminate>(FPlatformProcess::UserName(false)), EngineAddress);
	}
}


/* FSessionInstanceInfo callbacks
 *****************************************************************************/

void FSessionInstanceInfo::HandleSessionLogMessage(const FSessionServiceLog& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	TSharedRef<FSessionLogMessage> LogMessage = MakeShareable(
		new FSessionLogMessage(
			InstanceId,
			InstanceName,
			Message.TimeSeconds,
			Message.Data,
			(ELogVerbosity::Type)Message.Verbosity,
			Message.Category
		)
	);

	LogMessages.Add(LogMessage);
	LogReceivedEvent.Broadcast(AsShared(), LogMessage);
}
