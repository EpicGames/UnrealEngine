// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "IMessageContext.h"
#include "IMessageBus.h"
#include "MessageEndpoint.h"
#include "ISessionInstanceInfo.h"
#include "ISessionInfo.h"

struct FEngineServicePong;
struct FSessionLogMessage;
struct FSessionServiceLog;
struct FSessionServicePong;

/**
 * Implements a class to maintain all info related to a game instance in a session
 */
class FSessionInstanceInfo
	: public TSharedFromThis<FSessionInstanceInfo>
	, public ISessionInstanceInfo
{
public:

	/** Default constructor. */
	FSessionInstanceInfo() { }
	
	/** Destructor. */
	virtual ~FSessionInstanceInfo();

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InInstanceId The instance's identifier.
	 * @param InOwner The session that owns this instance.
	 * @param InMessageBus The message bus to use.
	 */
	FSessionInstanceInfo(const FGuid& InInstanceId, const TSharedRef<ISessionInfo>& InOwner, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus);

public:

	/**
	 * Updates this instance info with the data in the specified message.
	 *
	 * @param Message The message containing engine information.
	 * @param Context The message context.
	 */
	void UpdateFromMessage(const FEngineServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/**
	 * Updates this instance info with the data in the specified message.
	 *
	 * @param Message The message containing instance information.
	 * @param Context The message context.
	 */
	void UpdateFromMessage(const FSessionServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

public:	

	//~ IGameInstanceInfo interface

	virtual void ExecuteCommand(const FString& CommandString) override;

	virtual const FString& GetBuildDate() const override
	{
		return BuildDate;
	}

	virtual const FString& GetCurrentLevel() const override
	{
		return CurrentLevel;
	}

	virtual const FString& GetDeviceName() const override
	{
		return DeviceName;
	}

	virtual int32 GetEngineVersion() const override
	{
		return EngineVersion;
	}

	virtual const FGuid& GetInstanceId() const override
	{
		return InstanceId;
	}

	virtual const FString& GetInstanceName() const override
	{
		return InstanceName;
	}

	virtual const FString& GetInstanceType() const override
	{
		return InstanceType;
	}

	virtual const FDateTime& GetLastUpdateTime() const override
	{
		return LastUpdateTime;
	}

	virtual const TArray<TSharedPtr<FSessionLogMessage>>& GetLog() override
	{
		return LogMessages;
	}

	virtual TSharedPtr<ISessionInfo> GetOwnerSession() override
	{
		return Owner.Pin();
	}

	virtual const FString& GetPlatformName() const override
	{
		return PlatformName;
	}

	virtual float GetWorldTimeSeconds() const override
	{
		return WorldTimeSeconds;
	}

	virtual bool IsAuthorized() const override
	{
		return Authorized;
	}

	DECLARE_DERIVED_EVENT(FSessionInstanceInfo, ISessionInstanceInfo::FLogReceivedEvent, FLogReceivedEvent);
	virtual FLogReceivedEvent& OnLogReceived() override
	{
		return LogReceivedEvent;
	}

	virtual bool PlayHasBegun() const override
	{
		return HasBegunPlay;
	}

	virtual void Terminate() override;

private:

	/** Handles FSessionServiceLog messages. */
	void HandleSessionLogMessage(const FSessionServiceLog& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

private:

	/** Holds the message bus address of the application instance. */
	FMessageAddress ApplicationAddress;

	/** Whether the current user is authorized to interact with this instance. */
	bool Authorized = false;

	/** Holds the instance's build date. */
	FString BuildDate;

	/** Holds the instance's current level. */
	FString	CurrentLevel;

	/** Holds the device name. */
	FString DeviceName;

	/** Holds the message bus address of the engine instance. */
	FMessageAddress EngineAddress;

	/** Holds the instance's engine version. */
	int32 EngineVersion = 0;

	/** Holds a flag indicating whether the game has begun. */
	bool HasBegunPlay = false;

	/** Holds the instance identifier. */
	FGuid InstanceId;

	/** Holds the instance name. */
	FString InstanceName;

	/** Holds the instance type (i.e. game, editor etc.) */
	FString InstanceType;

	/** Holds the time at which the last pong was received. */
	FDateTime LastUpdateTime;

	/** Holds the collection of received log messages. */
	TArray<TSharedPtr<FSessionLogMessage>> LogMessages;

	/** Holds the message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Holds a reference to the session that owns this instance. */
	TWeakPtr<ISessionInfo> Owner;

	/** Holds the name of the platform that the instance is running on. */
	FString PlatformName;

	/** Holds the instance's current game world time. */
	float WorldTimeSeconds = 0.0f;

private:

	/** Holds an event delegate that is executed when a log message was received from the remote session. */
	FLogReceivedEvent LogReceivedEvent;
};
