// Copyright Epic Games, Inc. All Rights Reserved.

#include "SessionManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CommandLine.h"
#include "Containers/Ticker.h"
#include "EngineServiceMessages.h"
#include "MessageEndpointBuilder.h"
#include "SessionServiceMessages.h"
#include "SessionInfo.h"

/** Defines the interval in seconds in which devices are being pinged by the proxy manager. */
#define SESSION_MANAGER_PING_INTERVAL 5.0f

/* FSessionManager structors
 *****************************************************************************/

FSessionManager::FSessionManager(const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus)
	: MessageBusPtr(InMessageBus)
{
	// fill in the owner array
	FString Filter;

	if (FParse::Value(FCommandLine::Get(), TEXT("SessionFilter="), Filter))
	{
		// Allow support for -SessionFilter=Filter1+Filter2+Filter3
		int32 PlusIdx = Filter.Find(TEXT("+"), ESearchCase::CaseSensitive);

		while (PlusIdx != INDEX_NONE)
		{
			FString Owner = Filter.Left(PlusIdx);
			FilteredOwners.Add(Owner);
			Filter.RightInline(Filter.Len() - (PlusIdx + 1), EAllowShrinking::No);
			PlusIdx = Filter.Find(TEXT("+"), ESearchCase::CaseSensitive);
		}

		FilteredOwners.Add(Filter);
	}

	if (!FParse::Value(FCommandLine::Get(), TEXT("SessionPingInterval="), PingInterval))
	{
		PingInterval = SESSION_MANAGER_PING_INTERVAL;
	}

	// connect to message bus
	MessageEndpoint = FMessageEndpoint::Builder("FSessionManager", InMessageBus)
		.Handling<FEngineServicePong>(this, &FSessionManager::HandleEnginePongMessage)
		.Handling<FSessionServicePong>(this, &FSessionManager::HandleSessionPongMessage);

	// initialize ticker
	TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FSessionManager::HandleTicker), PingInterval);

	SendPing();
}


FSessionManager::~FSessionManager()
{
	FTSTicker::RemoveTicker(TickDelegateHandle);
	FMessageEndpoint::SafeRelease(MessageEndpoint);
}


/* ISessionManager interface
 *****************************************************************************/

void FSessionManager::AddOwner(const FString& InOwner)
{
	FilteredOwners.Add(InOwner);
}


const TArray<TSharedPtr<ISessionInstanceInfo>>& FSessionManager::GetSelectedInstances() const
{
	return SelectedInstances;
}


const TSharedPtr<ISessionInfo>& FSessionManager::GetSelectedSession() const
{
	return SelectedSession;
}


void FSessionManager::GetSessions(TArray<TSharedPtr<ISessionInfo>>& OutSessions) const
{
	OutSessions.Empty(Sessions.Num());

	for (TMap<FGuid, TSharedPtr<FSessionInfo> >::TConstIterator It(Sessions); It; ++It)
	{
		OutSessions.Add(It.Value());
	}
}


bool FSessionManager::IsInstanceSelected(const TSharedRef<ISessionInstanceInfo>& Instance) const
{
	return ((Instance->GetOwnerSession() == SelectedSession) && SelectedInstances.Contains(Instance));
}


FSimpleMulticastDelegate& FSessionManager::OnSessionsUpdated()
{
	return SessionsUpdatedDelegate;
}


FSimpleMulticastDelegate& FSessionManager::OnSessionInstanceUpdated()
{
	return SessionInstanceUpdatedDelegate;
}


void FSessionManager::RemoveOwner(const FString& InOwner)
{
	FilteredOwners.Remove(InOwner);
	RefreshSessions();
}


bool FSessionManager::SelectSession(const TSharedPtr<ISessionInfo>& Session)
{
	// already selected?
	if (Session == SelectedSession)
	{
		return true;
	}

	// do we own the session?
	if (Session.IsValid() && !Sessions.Contains(Session->GetSessionId()))
	{
		return false;
	}

	// allowed to de/select?
	bool CanSelect = true;
	CanSelectSessionDelegate.Broadcast(Session, CanSelect);

	if (!CanSelect)
	{
		return false;
	}

	// Notify listeners that previously selected sessions have been deselected.
	for (TSharedPtr<ISessionInstanceInfo> Instance : SelectedInstances)
	{
		InstanceSelectionChangedDelegate.Broadcast(Instance, false);
	}

	// set selection
	SelectedInstances.Empty();
	SelectedSession = Session;
	SelectedSessionChangedEvent.Broadcast(Session);

	return true;
}


bool FSessionManager::SetInstanceSelected(const TSharedRef<ISessionInstanceInfo>& Instance, bool Selected)
{
	if (Instance->GetOwnerSession() != SelectedSession)
	{
		return false;
	}

	if (Selected)
	{
		if (!SelectedInstances.Contains(Instance))
		{
			SelectedInstances.Add(Instance);
			InstanceSelectionChangedDelegate.Broadcast(Instance, true);
		}
	}
	else
	{
		if (SelectedInstances.Remove(Instance) > 0)
		{
			InstanceSelectionChangedDelegate.Broadcast(Instance, false);
		}
	}

	return true;
}


/* FSessionManager implementation
 *****************************************************************************/

void FSessionManager::FindExpiredSessions(const FDateTime& Now)
{
	bool Dirty = false;

	for (TMap<FGuid, TSharedPtr<FSessionInfo> >::TIterator It(Sessions); It; ++It)
	{
		if (Now > It.Value()->GetLastUpdateTime() + FTimespan::FromSeconds(10.0))
		{
			It.RemoveCurrent();
			Dirty = true;
		}
	}

	if (Dirty)
	{
		SessionsUpdatedDelegate.Broadcast();
	}
}


bool FSessionManager::IsValidOwner(const FString& Owner)
{
	if (Owner == FPlatformProcess::UserName(false))
	{
		return true;
	}

	for (const auto& FilteredOwner : FilteredOwners)
	{
		if (FilteredOwner == Owner)
		{
			return true;
		}
	}

	return false;
}


void FSessionManager::RefreshSessions()
{
	bool Dirty = false;

	for (TMap<FGuid, TSharedPtr<FSessionInfo> >::TIterator It(Sessions); It; ++It)
	{
		if (!IsValidOwner(It.Value()->GetSessionOwner()))
		{
			It.RemoveCurrent();
			Dirty = true;
		}
	}

	if (Dirty)
	{
		SessionsUpdatedDelegate.Broadcast();
	}
}


void FSessionManager::SendPing()
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FSessionmanager_SendPing);

	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FEngineServicePing>(), EMessageScope::Network);
		MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FSessionServicePing>(FPlatformProcess::UserName(false)), EMessageScope::Network);
	}

	LastPingTime = FDateTime::UtcNow();
}


/* FSessionManager callbacks
 *****************************************************************************/

void FSessionManager::HandleEnginePongMessage(const FEngineServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (!Message.SessionId.IsValid())
	{
		return;
	}

	// update instance
	TSharedPtr<FSessionInfo> Session = Sessions.FindRef(Message.SessionId);

	if (Session.IsValid())
	{
		Session->UpdateFromMessage(Message, Context);
		SessionInstanceUpdatedDelegate.Broadcast();
	}
}


void FSessionManager::HandleLogReceived(const TSharedRef<ISessionInfo>& Session, const TSharedRef<ISessionInstanceInfo>& Instance, const TSharedRef<FSessionLogMessage>& Message)
{
	if (Session == SelectedSession)
	{
		LogReceivedEvent.Broadcast(Session, Instance, Message);
	}
}


void FSessionManager::HandleSessionPongMessage(const FSessionServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (!Message.SessionId.IsValid())
	{
		return;
	}

	if (!Message.Standalone && !IsValidOwner(Message.SessionOwner))
	{
		return;
	}

	auto MessageBus = MessageBusPtr.Pin();

	if (!MessageBus.IsValid())
	{
		return;
	}

	// update session
	TSharedPtr<FSessionInfo>& Session = Sessions.FindOrAdd(Message.SessionId);

	if (Session.IsValid())
	{
		if (Session->GetSessionOwner() != Message.SessionOwner)
		{
			Session->UpdateFromMessage(Message, Context);
			SessionsUpdatedDelegate.Broadcast();
		}
		else
		{
			Session->UpdateFromMessage(Message, Context);
		}
	}
	else
	{
		Session = MakeShareable(new FSessionInfo(Message.SessionId, MessageBus.ToSharedRef()));
		Session->OnLogReceived().AddSP(this, &FSessionManager::HandleLogReceived);
		Session->UpdateFromMessage(Message, Context);

		SessionsUpdatedDelegate.Broadcast();
	}
}


bool FSessionManager::HandleTicker(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FSessionmanager_HandleTicker);

	FDateTime Now = FDateTime::UtcNow();

	// @todo gmp: don't expire sessions for now
//	FindExpiredSessions(Now);

	if (Now >= LastPingTime + FTimespan::FromSeconds(PingInterval))
	{
		SendPing();
	}

	return true;
}

TSharedPtr<ISessionInstanceInfo> FSessionManager::GetInstance(const FGuid& Id) const
{
	for (const auto& SessionPair : Sessions)
	{
		const auto& Session = SessionPair.Value;
		TArray<TSharedPtr<ISessionInstanceInfo>> Instances;
		Session->GetInstances(Instances);
		for (const auto& Instance : Instances)
		{
			if (Instance->GetInstanceId() == Id)
			{
				return Instance;
			}
		}
	}
	return nullptr;
}
