// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "IMessageContext.h"
#include "IMessageBus.h"
#include "MessageEndpoint.h"
#include "ISessionInstanceInfo.h"
#include "ISessionInfo.h"
#include "SessionLogMessage.h"
#include "ISessionManager.h"
#include "Containers/Ticker.h"

class FSessionInfo;
struct FEngineServicePong;
struct FSessionServicePong;

/**
 * Implement the session manager
 */
class FSessionManager
	: public TSharedFromThis<FSessionManager>
	, public ISessionManager
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InMessageBus The message bus to use.
	 */
	FSessionManager(const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus);
	
	/** Destructor */
	~FSessionManager();

public:

	//~ ISessionManager interface

	virtual void AddOwner(const FString& InOwner) override;
	virtual const TArray<TSharedPtr<ISessionInstanceInfo>>& GetSelectedInstances() const override;
	virtual const TSharedPtr<ISessionInfo>& GetSelectedSession() const override;
	virtual void GetSessions(TArray<TSharedPtr<ISessionInfo>>& OutSessions) const override;
	virtual bool IsInstanceSelected(const TSharedRef<ISessionInstanceInfo>& Instance) const override;

	DECLARE_DERIVED_EVENT(FSessionManager, ISessionManager::FCanSelectSessionEvent, FCanSelectSessionEvent)
	virtual FCanSelectSessionEvent& OnCanSelectSession() override
	{
		return CanSelectSessionDelegate;
	}

	DECLARE_DERIVED_EVENT(FSessionManager, ISessionManager::FInstanceSelectionChangedEvent, FInstanceSelectionChangedEvent)
	virtual FInstanceSelectionChangedEvent& OnInstanceSelectionChanged() override
	{
		return InstanceSelectionChangedDelegate;
	}

	DECLARE_DERIVED_EVENT(FSessionManager, ISessionManager::FLogReceivedEvent, FLogReceivedEvent)
	virtual FLogReceivedEvent& OnLogReceived() override
	{
		return LogReceivedEvent;
	}

	DECLARE_DERIVED_EVENT(FSessionManager, ISessionManager::FSelectedSessionChangedEvent, FSelectedSessionChangedEvent)
	virtual FSelectedSessionChangedEvent& OnSelectedSessionChanged() override
	{
		return SelectedSessionChangedEvent;
	}

	virtual FSimpleMulticastDelegate& OnSessionsUpdated() override;
	virtual FSimpleMulticastDelegate& OnSessionInstanceUpdated() override;
	virtual void RemoveOwner(const FString& InOwner) override;
	virtual bool SelectSession(const TSharedPtr<ISessionInfo>& Session) override;
	virtual bool SetInstanceSelected(const TSharedRef<ISessionInstanceInfo>& Instance, bool Selected) override;

protected:

	/**
	 * Finds and removes sessions that haven't been updated in a while.
	 *
	 * @param Now The current time.
	 */
	void FindExpiredSessions(const FDateTime& Now);

	/**
	 * Checks whether the specified owner is valid.
	 *
	 * @return true if the owner is valid, false otherwise.
	 */
	bool IsValidOwner(const FString& Owner);

	/** Refresh the sessions based on the owner filter list. */
	void RefreshSessions();

	/** Pings all sessions on the network. */
	void SendPing();

private:

	/** Callback for handling FSessionServicePong messages. */
	void HandleEnginePongMessage(const FEngineServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Callback received log entries. */
	void HandleLogReceived(const TSharedRef<ISessionInfo>& Session, const TSharedRef<ISessionInstanceInfo>& Instance, const TSharedRef<FSessionLogMessage>& Message);

	/** Callback for handling FSessionServicePong messages. */
	void HandleSessionPongMessage(const FSessionServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Callback for ticks from the ticker. */
	bool HandleTicker(float DeltaTime);

	virtual TSharedPtr<ISessionInstanceInfo> GetInstance(const FGuid& Id) const override;

private:

	/** The address of the automation controller to where we can forward any automation workers found. */
	FGuid AutomationControllerAddress;

	/** Holds the time at which the last ping was sent. */
	FDateTime LastPingTime;

	/** Holds a pointer to the message bus. */
	TWeakPtr<IMessageBus, ESPMode::ThreadSafe> MessageBusPtr;

	/** Holds the messaging endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Holds the list of currently selected instances. */
	TArray<TSharedPtr<ISessionInstanceInfo>> SelectedInstances;

	/** Holds a reference to the currently selected session. */
	TSharedPtr<ISessionInfo> SelectedSession;

	/** Holds the collection of discovered sessions. */
	TMap<FGuid, TSharedPtr<FSessionInfo>> Sessions;

private:

	/** Holds a delegate to be invoked before a session is selected. */
	FCanSelectSessionEvent CanSelectSessionDelegate;

	/** Holds a delegate to be invoked when an instance changes its selection state. */
	FInstanceSelectionChangedEvent InstanceSelectionChangedDelegate;

	/** Owner filter list. */
	TArray<FString> FilteredOwners;

	/** Holds a delegate to be invoked when the selected session received a log message. */
	FLogReceivedEvent LogReceivedEvent;

	/** Holds a delegate to be invoked when the selected session changed. */
	FSelectedSessionChangedEvent SelectedSessionChangedEvent;

	/** Holds a delegate to be invoked when the session list was updated. */
	FSimpleMulticastDelegate SessionsUpdatedDelegate;

	/** Holds a delegate to be invoked when a session instance is updated. */
	FSimpleMulticastDelegate SessionInstanceUpdatedDelegate;

	/** Holds a delegate to be invoked when the widget ticks. */
	FTSTicker::FDelegateHandle TickDelegateHandle;

	/** Holds current ping interval. */
	float PingInterval;
};
