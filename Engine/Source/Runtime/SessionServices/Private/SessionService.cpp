// Copyright Epic Games, Inc. All Rights Reserved.

#include "SessionService.h"
#include "MessageEndpointBuilder.h"
#include "Misc/ScopeLock.h"
#include "Misc/App.h"
#include "SessionServiceMessages.h"
#include "Misc/ScopeExit.h"


/* FSessionService structors
 *****************************************************************************/

FSessionService::FSessionService(const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus)
	: MessageBusPtr(InMessageBus)
	, IsInSendLog(false)
{ }


FSessionService::~FSessionService()
{
	Stop();
}


/* ISessionService interface
 *****************************************************************************/

bool FSessionService::Start()
{
	auto MessageBus = MessageBusPtr.Pin();

	if (!MessageBus.IsValid())
	{
		return false;
	}

	// initialize messaging
	MessageEndpoint = FMessageEndpoint::Builder("FSessionService", MessageBus.ToSharedRef())
		.Handling<FSessionServiceLogSubscribe>(this, &FSessionService::HandleSessionLogSubscribeMessage)
		.Handling<FSessionServiceLogUnsubscribe>(this, &FSessionService::HandleSessionLogUnsubscribeMessage)
		.Handling<FSessionServicePing>(this, &FSessionService::HandleSessionPingMessage);

	if (!MessageEndpoint.IsValid())
	{
		return false;
	}

	MessageEndpoint->Subscribe<FSessionServicePing>();
	GLog->AddOutputDevice(this);

	return true;
}


void FSessionService::Stop()
{
	if (IsRunning())
	{
		GLog->RemoveOutputDevice(this);
		MessageEndpoint.Reset();
	}
}


/* FSessionService implementation
 *****************************************************************************/

void FSessionService::SendLog(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	if (!MessageEndpoint.IsValid())
	{
		return;
	}

	FScopeLock Lock(&LogSubscribersLock);

	// Guard against going recursive from logging in messaging code
	if (!IsInSendLog)
	{
		TGuardValue<bool> Guard(IsInSendLog, true);
		
		if (LogSubscribers.Num() > 0)
		{
			MessageEndpoint->Send(
				FMessageEndpoint::MakeMessage<FSessionServiceLog>(
					Category,
					Data,
					FApp::GetInstanceId(),
					FPlatformTime::Seconds() - GStartTime,
					Verbosity
				),
				LogSubscribers
			);
		}	
	}
}


void FSessionService::SendNotification(const TCHAR* NotificationText, const FMessageAddress& Recipient)
{
	if (!MessageEndpoint.IsValid())
	{
		return;
	}

	MessageEndpoint->Send(
		FMessageEndpoint::MakeMessage<FSessionServiceLog>(
			FName("RemoteSession"),
			NotificationText,
			FApp::GetInstanceId(),
			FPlatformTime::Seconds() - GStartTime,
			ELogVerbosity::Display
		),
		Recipient
	);
}


void FSessionService::SendPong(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FString& UserName)
{
	if (!MessageEndpoint.IsValid())
	{
		return;
	}

	FSessionServicePong* Message = FMessageEndpoint::MakeMessage<FSessionServicePong>();
	{
		Message->Authorized = FApp::IsAuthorizedUser(UserName);
		Message->BuildDate = FApp::GetBuildDate();
		Message->DeviceName = FPlatformProcess::ComputerName();
		Message->InstanceId = FApp::GetInstanceId();
		Message->InstanceName = FApp::GetInstanceName();
		Message->PlatformName = FPlatformProperties::PlatformName();
		Message->SessionId = FApp::GetSessionId();
		Message->SessionName = FApp::GetSessionName();
		Message->SessionOwner = FApp::GetSessionOwner();
		Message->Standalone = FApp::IsStandalone();
	}

	MessageEndpoint->Send(Message, Context->GetSender());
}


/* FSessionService callbacks
 *****************************************************************************/

void FSessionService::HandleMessageEndpointShutdown()
{
	MessageEndpoint.Reset();
}


void FSessionService::HandleSessionLogSubscribeMessage(const FSessionServiceLogSubscribe& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FScopeLock Lock(&LogSubscribersLock);
	LogSubscribers.AddUnique(Context->GetSender());
}


void FSessionService::HandleSessionLogUnsubscribeMessage(const FSessionServiceLogUnsubscribe& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	FScopeLock Lock(&LogSubscribersLock);
	LogSubscribers.Remove(Context->GetSender());
}


void FSessionService::HandleSessionPingMessage(const FSessionServicePing& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	SendPong(Context, Message.UserName);
}
