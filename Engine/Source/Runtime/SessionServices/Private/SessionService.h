// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "IMessageBus.h"
#include "ISessionService.h"
#include "MessageEndpoint.h"

struct FSessionServiceLogSubscribe;
struct FSessionServiceLogUnsubscribe;
struct FSessionServicePing;


/**
 * Implements an application session service.
 */
class FSessionService
	: public FOutputDevice
	, public ISessionService
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InMessageBus The message bus to use.
	 */
	FSessionService(const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus);

	/** Virtual destructor. */
	virtual ~FSessionService();

public:

	//~ FOutputDevice interface

	virtual void Serialize(const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		SendLog(Data, Verbosity, Category);
	}

public:

	//~ ISessionService interface

	virtual bool IsRunning() override
	{
		return MessageEndpoint.IsValid();
	}

	virtual bool Start() override;
	virtual void Stop() override;

protected:

	/**
	 * Sends a log message to subscribed recipients.
	 *
	 * @param Data The log message data.
	 * @param Verbosity The verbosity type.
	 * @param Category The log category.
	 */
	void SendLog(const TCHAR* Data, ELogVerbosity::Type Verbosity = ELogVerbosity::Log, const class FName& Category = "Log");

	/**
	 * Sends a notification to the specified recipient.
	 *
	 * @param NotificationText The notification text.
	 * @param Recipient The recipient's message address.
	 */
	void SendNotification(const TCHAR* NotificationText, const FMessageAddress& Recipient);

	/**
	 * Publishes a ping response.
	 *
	 * @param Context The context of the received Ping message.
	 * @param UserName The name of the user that sent the ping.
	 */
	void SendPong(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FString& UserName);

private:

	/** Handles message bus shutdowns. */
	void HandleMessageEndpointShutdown();

	/** Handles FSessionServiceLogSubscribe messages. */
	void HandleSessionLogSubscribeMessage(const FSessionServiceLogSubscribe& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FSessionServiceLogUnsubscribe messages. */
	void HandleSessionLogUnsubscribeMessage(const FSessionServiceLogUnsubscribe& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Handles FSessionServicePing messages. */
	void HandleSessionPingMessage(const FSessionServicePing& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

private:

	/** Holds the list of log subscribers. */
	TArray<FMessageAddress> LogSubscribers;

	/** Holds a critical section for the log subscribers array. */
	FCriticalSection LogSubscribersLock;

	/** Holds a weak pointer to the message bus. */
	TWeakPtr<IMessageBus, ESPMode::ThreadSafe> MessageBusPtr;

	/** Holds the message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Set to true while in SendLog, used to avoid recursion */
	bool IsInSendLog;
};
