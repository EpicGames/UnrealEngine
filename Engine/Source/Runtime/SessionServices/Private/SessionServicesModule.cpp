// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "IMessagingModule.h"
#include "ISessionManager.h"
#include "ISessionService.h"
#include "ISessionServicesModule.h"
#include "Misc/App.h"
#include "Misc/CoreMisc.h"
#include "Modules/ModuleManager.h"
#include "SessionManager.h"
#include "SessionService.h"
#include "TraceController.h"

/**
 * Implements the SessionServices module.
 *
 * @todo gmp: needs better error handling in case MessageBusPtr is invalid
 */
class FSessionServicesModule
	: public FSelfRegisteringExec
	, public ISessionServicesModule
{
public:

	/** Default constructor. */
	FSessionServicesModule()
		: SessionManager(nullptr)
		, SessionService(nullptr)
	{ }

protected:

	//~ FSelfRegisteringExec interface

	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (!FParse::Command(&Cmd, TEXT("SESSION")))
		{
			return false;
		}

		if (FParse::Command(&Cmd, TEXT("AUTH")))
		{
			FApp::AuthorizeUser(FParse::Token(Cmd, false));
		}
		else if (FParse::Command(&Cmd, TEXT("DENY")))
		{
			FApp::DenyUser(FParse::Token(Cmd, false));
		}
		else if (FParse::Command(&Cmd, TEXT("DENYALL")))
		{
			FApp::DenyAllUsers();
		}
		else if (FParse::Command(&Cmd, TEXT("STATUS")))
		{
			const FCoreTexts& CoreTexts = FCoreTexts::Get();

			Ar.Logf(TEXT("Instance ID: %s"), *FApp::GetInstanceId().ToString());
			Ar.Logf(TEXT("Instance Name: %s"), *FApp::GetInstanceName());
			Ar.Logf(TEXT("Session ID: %s"), *FApp::GetSessionId().ToString());
			Ar.Logf(TEXT("Session Name: %s"), *FApp::GetSessionName());
			Ar.Logf(TEXT("Session Owner: %s"), *FApp::GetSessionOwner());
			Ar.Logf(TEXT("Standalone: %s"), FApp::IsStandalone() ? *CoreTexts.Yes.ToString() : *CoreTexts.No.ToString());
		}
		else if (FParse::Command(&Cmd, TEXT("SETNAME")))
		{
			FApp::SetSessionName(FParse::Token(Cmd, false));
		}
		else if (FParse::Command(&Cmd, TEXT("SETOWNER")))
		{
			FApp::SetSessionOwner(FParse::Token(Cmd, false));
		}
		else
		{
			// show usage
			Ar.Log(TEXT("Usage: SESSION <Command>"));
			Ar.Log(TEXT(""));
			Ar.Log(TEXT("Command"));
			Ar.Log(TEXT("    AUTH <UserName> = Authorize a user to interact with this instance"));
			Ar.Log(TEXT("    DENY <UserName> = Unauthorize a user from interacting with this instance"));
			Ar.Log(TEXT("    DENYALL = Removes all authorized session users"));
			Ar.Log(TEXT("    SETNAME <Name> = Sets the name of this instance"));
			Ar.Log(TEXT("    SETOWNER <Owner> = Sets the name of the owner of this instance"));
			Ar.Log(TEXT("    STATUS = Displays the status of this application session"));
		}

		return true;
	}

public:

	//~ ISessionServicesModule interface

	virtual TSharedPtr<ISessionManager> GetSessionManager() override
	{
		if (!SessionManager.IsValid() && MessageBusPtr.IsValid())
		{
			SessionManager = MakeShareable(new FSessionManager(MessageBusPtr.Pin().ToSharedRef()));
		}

		return SessionManager;
	}

	virtual TSharedPtr<ISessionService> GetSessionService() override
	{
		if (!SessionService.IsValid() && MessageBusPtr.IsValid())
		{
			SessionService = MakeShareable(new FSessionService(MessageBusPtr.Pin().ToSharedRef()));
		}

		return SessionService;
	}

	virtual TSharedPtr<ITraceController> GetTraceController() override
	{
		if (!TraceController.IsValid() && MessageBusPtr.IsValid())
		{
			TraceController = MakeShareable(new FTraceController(MessageBusPtr.Pin().ToSharedRef()));
		}

		return TraceController;
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		MessageBusPtr = IMessagingModule::Get().GetDefaultBus();
	}

	virtual void ShutdownModule() override
	{
		TraceController.Reset();
		SessionManager.Reset();
		SessionService.Reset();
	}

private:
	
	/** Holds a weak pointer to the message bus. */
	TWeakPtr<IMessageBus, ESPMode::ThreadSafe> MessageBusPtr;

	/** Holds the session manager singleton. */
	TSharedPtr<ISessionManager> SessionManager;

	/** Holds the session service singleton. */
	TSharedPtr<ISessionService> SessionService;

	/** Holds the control for Tracing */
	TSharedPtr<FTraceController> TraceController;
};


IMPLEMENT_MODULE(FSessionServicesModule, SessionServices);
