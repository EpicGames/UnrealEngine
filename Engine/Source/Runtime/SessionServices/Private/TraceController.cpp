// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceController.h"

#include "IMessageBus.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "TraceControlMessages.h"


FTraceController::FTraceController(const TSharedRef<IMessageBus>& InMessageBus)
	: MessageBus(InMessageBus)
{
	MessageEndpoint = FMessageEndpoint::Builder("FTraceController", InMessageBus)
	  .Handling<FTraceControlDiscovery>(this, &FTraceController::OnDiscoveryResponse)
	  .Handling<FTraceControlStatus>(this, &FTraceController::OnStatus)
	  .Handling<FTraceControlSettings>(this, &FTraceController::OnSettings)
	  .Handling<FTraceControlChannelsDesc>(this, &FTraceController::OnChannelsDesc)
	  .Handling<FTraceControlChannelsStatus>(this, &FTraceController::OnChannelsStatus)
	  .NotificationHandling(FOnBusNotification::CreateRaw(this, &FTraceController::OnNotification));
}

FTraceController::~FTraceController()
{
}

void FTraceController::SendDiscoveryRequest(const FGuid& SessionId, const FGuid& InstanceId) const
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlDiscoveryPing>();
	Message->SessionId = SessionId;
	Message->InstanceId = InstanceId;
	MessageEndpoint->Publish<FTraceControlDiscoveryPing>(Message);
}

void FTraceController::SendDiscoveryRequest()
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlDiscoveryPing>();
	MessageEndpoint->Publish<FTraceControlDiscoveryPing>(Message);
}

void FTraceController::SendStatusUpdateRequest()
{
	MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FTraceControlStatusPing>());
}

void FTraceController::SendChannelUpdateRequest()
{
	FReadScopeLock _(InstancesLock);
	
	for (const auto& Instance : Instances)
	{
		const auto Message = FMessageEndpoint::MakeMessage<FTraceControlChannelsPing>();
		Message->KnownChannelCount = uint32(Instance.Value.Status.Channels.Num());
		MessageEndpoint->Send(Message, Instance.Key);
	}
}

void FTraceController::SendSettingsUpdateRequest()
{
	for (const auto& Instance : Instances)
	{
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FTraceControlSettingsPing>(), Instance.Key);
	}
}

void FTraceController::OnNotification(const FMessageBusNotification& Event)
{
	if (Event.NotificationType == EMessageBusNotification::Unregistered)
	{
		// Many endpoints may be removed from one instance, look for the
		// one we have registered.
		FWriteScopeLock _(InstancesLock);
		if(Instances.Remove(Event.RegistrationAddress) > 0)
		{
			InstanceToAddress = InstanceToAddress.FilterByPredicate([&](auto It) { return It.Value != Event.RegistrationAddress; } );
		}
	}
}

void FTraceController::OnDiscoveryResponse(const FTraceControlDiscovery& Message, const TSharedRef<IMessageContext>& Context)
{
	FWriteScopeLock _(InstancesLock);

	// If the message bus is no longer available there is no point in registering
	// more sessions.
	TSharedPtr<IMessageBus> Bus = MessageBus.Pin();
	if (!Bus)
	{
		return;
	}

	FTracingInstance* Instance = Instances.Find(Context->GetSender());
	if (!Instance)
	{
		// Create a new instance with default status
		Instance = &Instances.Emplace(Context->GetSender(), {Bus.ToSharedRef(), Context->GetSender(), this});
		InstanceToAddress.Add(Message.InstanceId, Context->GetSender());
	}
	
	UpdateStatus(Message, Instance->Status);

	// This is the application session id, which is not necessarily the
	// same as trace session (maybe overridden by commandline)
	Instance->Status.SessionId = Message.SessionId;
	Instance->Status.InstanceId = Message.InstanceId;
	Instance->LastMessageTimestamp = FDateTime::Now();

	FTraceStatus::EUpdateType UpdateType = FTraceStatus::EUpdateType::ChannelsDesc | FTraceStatus::EUpdateType::ChannelsStatus | FTraceStatus::EUpdateType::Status;
	StatusReceivedEvent.Broadcast(Instance->Status, UpdateType, Instance->Commands);
}

void FTraceController::OnStatus(const FTraceControlStatus& Message, const TSharedRef<IMessageContext>& Context)
{
	FWriteScopeLock _(InstancesLock);
	
	if (FTracingInstance* Instance = Instances.Find(Context->GetSender()))
	{
		Instance->LastMessageTimestamp = FDateTime::Now();

		FTraceStatus& Status = Instance->Status;
		UpdateStatus(Message, Status);

		StatusReceivedEvent.Broadcast(Status, FTraceStatus::EUpdateType::Status, Instance->Commands);
	}
}

void FTraceController::OnChannelsDesc(const FTraceControlChannelsDesc& Message, const TSharedRef<IMessageContext>& Context)
{
	FWriteScopeLock _(InstancesLock);
	
	const auto Instance = Instances.Find(Context->GetSender());
	
	if (!Instance)
	{
		return;
	}

	Instance->LastMessageTimestamp = FDateTime::Now();

	FTraceStatus& Status = Instance->Status;
	check(Message.Channels.Num() == Message.Ids.Num() && Message.Channels.Num() == Message.Descriptions.Num());
	const int32 Count = Message.Channels.Num();
	for(int32 Index = 0; Index < Count; ++Index)
	{
		const uint32 Id = Message.Ids[Index];
		if (Status.Channels.Contains(Id))
		{
			continue;
		}

		FTraceStatus::FChannel& NewChannel = Status.Channels.Add(Id);
		NewChannel.Name = Message.Channels[Index];
		NewChannel.Description = Message.Descriptions[Index];
		NewChannel.Id = Id;
		NewChannel.bEnabled = false;
		NewChannel.bReadOnly = Message.ReadOnlyIds.Contains(Id);
		
		StatusReceivedEvent.Broadcast(Status, FTraceStatus::EUpdateType::ChannelsDesc, Instance->Commands);
	}

	// Allow the commands instance to update its list of channels
	Instance->Commands.OnChannelsDesc(Message);
}

void FTraceController::OnChannelsStatus(const FTraceControlChannelsStatus& Message, const TSharedRef<IMessageContext>& Context)
{
	FWriteScopeLock _(InstancesLock);
	
	if (const auto Instance = Instances.Find(Context->GetSender()))
	{
		Instance->LastMessageTimestamp = FDateTime::Now();

		FTraceStatus& Status = Instance->Status;
		for (auto& [Id, Channel] : Status.Channels)
		{
			Channel.bEnabled = Message.EnabledIds.Contains(Id);
		}
		
		StatusReceivedEvent.Broadcast(Status, FTraceStatus::EUpdateType::ChannelsStatus, Instance->Commands);
	}
}

void FTraceController::OnSettings(const FTraceControlSettings& Message, const TSharedRef<IMessageContext>& Context)
{
	FWriteScopeLock _(InstancesLock);
	
	if (const auto Instance = Instances.Find(Context->GetSender()))
	{
		Instance->LastMessageTimestamp = FDateTime::Now();

		FTraceStatus& Status = Instance->Status;
		FTraceStatus::FSettings& Settings = Status.Settings;
		Settings.bUseImportantCache = Message.bUseImportantCache;
		Settings.bUseWorkerThread = Message.bUseWorkerThread;
		Settings.TailSizeBytes = Message.TailSizeBytes;
		Algo::Transform(Message.Capabilities, Settings.Capabilities, [](const FTraceCapability& In) { 
				return FTraceStatus::FCapability { In.Name }; 
		});

		Settings.ChannelPresets.Empty();
		for (const FTraceChannelPreset& Preset : Message.ChannelPresets)
		{
			Settings.ChannelPresets.Add(FTraceStatus::FChannelPreset(Preset.Name, Preset.ChannelList, Preset.bIsReadOnly));
		}
		
		StatusReceivedEvent.Broadcast(Status, FTraceStatus::EUpdateType::Settings, Instance->Commands);
	}
}

void FTraceController::OnSecureResponse(const FTraceControlSecureResponse& Message, const TSharedRef<IMessageContext>& Context)
{
	FWriteScopeLock _(InstancesLock);
	
	if (const auto Instance = Instances.Find(Context->GetSender()))
	{
		FSecureConnection Connection;
		Connection.Host = Message.Host;
		Connection.Port = Message.Port;
		Connection.AuthorizationToken = Message.AuthorizationToken;
		if (Message.Mode == ETraceSecureMode::SelfSignedCertAndAuthToken)
		{
			Connection.CertificateAuthority = Message.CertificateAuthority;
		}
		else if (Message.Mode == ETraceSecureMode::AuthToken)
		{
			Connection.Identity = Message.Identity;
		}
		SecureResponse.Broadcast(Instance->Status, Connection);
	}
}

bool FTraceController::HasAvailableInstance(const FGuid& InstanceId)
{
	return InstanceToAddress.Contains(InstanceId);
}

bool FTraceController::IsActive(FGuid InstanceId, double TimeoutSec) const
{
	if (const auto& InstanceAddress = InstanceToAddress.Find(InstanceId))
	{
		if (auto* Instance = Instances.Find(*InstanceAddress))
		{
			return (FDateTime::Now() - Instance->LastMessageTimestamp).GetTotalSeconds() <= TimeoutSec;
		}
	}

	return false;
}

bool FTraceController::HasCapability(FGuid InstanceId, FStringView Name) 
{
	FReadScopeLock _(InstancesLock);
	if (const auto& InstanceAddress = InstanceToAddress.Find(InstanceId))
	{
		if (auto* Instance = Instances.Find(*InstanceAddress))
		{
			for (const auto& Capability : Instance->Status.Settings.Capabilities)
			{
				if (Name.Equals(Capability.Name, ESearchCase::IgnoreCase))
				{
					return true;
				}
			}
		}
	}
	return false;
}

void FTraceController::UpdateStatus(const FTraceControlStatus& Message, FTraceStatus& Status)
{
	Status.TraceSystemStatus = static_cast<FTraceStatus::ETraceSystemStatus>(Message.TraceSystemStatus);
	Status.StatusTimestamp = Message.StatusTimestamp;
	Status.bIsTracing = Message.bIsTracing;
	Status.Endpoint = Message.Endpoint;
	Status.SessionGuid = Message.SessionGuid;
	Status.TraceGuid = Message.TraceGuid;
	Status.bIsPaused = Message.bIsPaused;
	Status.bAreStatNamedEventsEnabled = Message.bAreStatNamedEventsEnabled;
	Status.Stats.BytesSent = Message.BytesSent;
	Status.Stats.MemoryUsed = Message.MemoryUsed;
	Status.Stats.BytesTraced = Message.BytesTraced;
	Status.Stats.CacheAllocated = Message.CacheAllocated;
	Status.Stats.CacheUsed = Message.CacheUsed;
	Status.Stats.CacheWaste = Message.CacheWaste;
}

void FTraceController::WithInstance(FGuid InstanceId, FCallback Func)
{
	FReadScopeLock _(InstancesLock);
	if(const auto& InstanceAddress = InstanceToAddress.Find(InstanceId))
	{
		if(auto* Instance = Instances.Find(*InstanceAddress))
		{
			Func(Instance->Status, Instance->Commands);
		}
	}
}

FTraceController::FTracingInstance::FTracingInstance(const TSharedRef<IMessageBus>& InMessageBus, FMessageAddress InService, FTraceController* Controller)
	: Status()
	, Commands(InMessageBus, InService, Controller)
{
}

TSharedPtr<ITraceController> ITraceController::Create(TSharedPtr<IMessageBus>& InBus)
{
	return MakeShareable(new FTraceController(InBus.ToSharedRef()));
}
