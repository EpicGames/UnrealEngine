// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "ITraceController.h"
#include "MessageEndpoint.h"
#include "TraceControllerCommands.h"
#include "TraceControlMessages.h"

struct FTraceControlSettings;
struct FTraceControlStatus;
struct FTraceControlDiscovery;
class IMessageBus;
class FMessageEndpoint;

/**
 * Interface to control other sessions tracing.
 */
class FTraceController : public ITraceController 
{
public:
	FTraceController(const TSharedRef<IMessageBus>& InMessageBus);
	virtual ~FTraceController() override;

private:
	virtual void SendDiscoveryRequest(const FGuid& SessionId, const FGuid& InstanceId) const override;
	virtual void SendDiscoveryRequest() override;
	virtual void SendStatusUpdateRequest() override;
	virtual void SendChannelUpdateRequest() override;
	virtual void SendSettingsUpdateRequest() override;
	virtual bool HasAvailableInstance(const FGuid& InstanceId) override;
	virtual bool IsActive(FGuid InstanceId, double TimeoutSec = 5.0f) const override;
	virtual bool HasCapability(FGuid InstanceId, FStringView Name) override;
	virtual void WithInstance(FGuid InstanceId, FCallback Func) override;
	
	DECLARE_DERIVED_EVENT(FTraceController, ITraceController::FStatusReceivedEvent, FStatusReceivedEvent);
	virtual FStatusReceivedEvent& OnStatusReceived() override
	{
		return StatusReceivedEvent;
	}

	DECLARE_DERIVED_EVENT(FTraceController, ITraceController::FSecureResponseEvent, FSecureResponseEvent);
	virtual FSecureResponseEvent& OnSecureResponse() override
	{
		return SecureResponse;
	}

	/* Message handlers */
	void OnNotification(const FMessageBusNotification& MessageBusNotification);
	void OnDiscoveryResponse(const FTraceControlDiscovery& Message, const TSharedRef<IMessageContext>& Context);
	void OnStatus(const FTraceControlStatus& Message, const TSharedRef<IMessageContext>& Context);
	void OnChannelsDesc(const FTraceControlChannelsDesc& Message, const TSharedRef<IMessageContext>& Context);
	void OnChannelsStatus(const FTraceControlChannelsStatus& Message, const TSharedRef<IMessageContext>& Context);
	void OnSettings(const FTraceControlSettings& Message, const TSharedRef<IMessageContext>& Context);
	static void UpdateStatus(const FTraceControlStatus& Message, FTraceStatus& Status);
	void OnSecureResponse(const FTraceControlSecureResponse& Message, const TSharedRef<IMessageContext>& Context);

private:
	
	struct FTracingInstance
	{
		FTraceStatus Status;
		FTraceControllerCommands Commands;
		FDateTime LastMessageTimestamp;

		FTracingInstance(const TSharedRef<IMessageBus>& InMessageBus, FMessageAddress Service, FTraceController* Controller);
		FTracingInstance() = delete;
	};

	/**
	 * Needed to create command instances when new sessions are discovered. We don't need a ref counted
	 * pointer to the message bus.
	 */
	TWeakPtr<IMessageBus> MessageBus;

	/** Our own endpoint for messages */
	TSharedPtr<FMessageEndpoint> MessageEndpoint;

	/** Address of the runtime endpoint for trace controls */
	FMessageAddress TraceControlAddress;

	/** Event for status updates on any session */
	FStatusReceivedEvent StatusReceivedEvent;

	/** Event for secure connection responses */
	FSecureResponseEvent SecureResponse;

	/** Lock to protect access to Instances list */
	mutable FRWLock InstancesLock;

	/** Known instances with an active trace service */
	TMap<FMessageAddress, FTracingInstance> Instances;

	/** Secondary lookup from instance -> address */
	TMap<FGuid, FMessageAddress> InstanceToAddress;

	friend class FTraceControllerCommands;
};
