// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceControllerCommands.h"

#include "Hash/xxhash.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "TraceController.h"
#include "TraceControlMessages.h"
#include "TraceController.h"


//------------------------------------------------------------------------------------------
// Utilities
//------------------------------------------------------------------------------------------

uint64 HashName(FStringView Name)
{
	// Strip plurals and convert to upper case.
	TStringBuilder<48> TransformedBuffer;
	TransformedBuffer << (Name.EndsWith('s') ? Name.LeftChop(1) : Name);
	FCString::Strupr(TransformedBuffer.GetData(), TransformedBuffer.Len());
	
	return FXxHash64::HashBuffer(TransformedBuffer.GetData(), TransformedBuffer.Len() * sizeof(TCHAR)).Hash;
}

template<typename StringType>
TArray<uint32> ResolveChannelNames(TConstArrayView<StringType> ChannelNames, const TMap<uint64, uint32>& Channels)
{
	TArray<uint32> Ids;
	for (const auto& ChannelName : ChannelNames)
	{
		const uint64 Hash = HashName(FStringView(ChannelName));
		if (const auto FoundId = Channels.Find(Hash))
		{
			Ids.Add(*FoundId);
		}
	}
	return MoveTemp(Ids);
}

//------------------------------------------------------------------------------------------
// FTraceControllerCommands
//------------------------------------------------------------------------------------------

FTraceControllerCommands::FTraceControllerCommands(const TSharedPtr<IMessageBus>& InMessageBus, FMessageAddress Service, class FTraceController* Controller)
	: ServiceAddress(Service)
{
	TStringBuilder<128> EndpointName;
	EndpointName << TEXT("FTraceControllerCommands_") << Service.ToString();
	MessageEndpoint = FMessageEndpoint::Builder(EndpointName.ToString(), InMessageBus.ToSharedRef())
		.Handling<FTraceControlSecureResponse>(Controller, &FTraceController::OnSecureResponse)
		.Handling<FTraceControlStatus>(Controller, &FTraceController::OnStatus)
		.Handling<FTraceControlChannelsDesc>(Controller, &FTraceController::OnChannelsDesc)
		.Handling<FTraceControlChannelsStatus>(Controller, &FTraceController::OnChannelsStatus)
		.Handling<FTraceControlSettings>(Controller, &FTraceController::OnSettings);
}

FTraceControllerCommands::~FTraceControllerCommands()
{
}

void FTraceControllerCommands::SetChannels(TConstArrayView<FStringView> ChannelsToEnable, TConstArrayView<FStringView> ChannelsToDisable)
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlChannelsSet>();
	Message->ChannelIdsToEnable = ResolveChannelNames(ChannelsToEnable, SettableChannels);
	Message->ChannelIdsToDisable = ResolveChannelNames(ChannelsToDisable, SettableChannels);
	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::SetChannels(TConstArrayView<FString> ChannelsToEnable, TConstArrayView<FString> ChannelsToDisable)
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlChannelsSet>();
	Message->ChannelIdsToEnable = ResolveChannelNames(ChannelsToEnable, SettableChannels);
	Message->ChannelIdsToDisable = ResolveChannelNames(ChannelsToDisable, SettableChannels);
	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::Send(FStringView Host, FStringView Channels, bool bExcludeTail)
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlSend>();
	Message->Host = Host;
	Message->Channels = Channels;
	Message->bExcludeTail = bExcludeTail;

	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::File(FStringView File, FStringView Channels, bool bExcludeTail, bool bTruncateFile)
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlFile>();
	Message->File = File;
	Message->Channels = Channels;
	Message->bExcludeTail = bExcludeTail;
	Message->bTruncateFile = bTruncateFile;

	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::Stop()
{
	MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FTraceControlStop>(), ServiceAddress);
}

void FTraceControllerCommands::SnapshotSend(FStringView Host)
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlSnapshotSend>();
	Message->Host = Host;

	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::SnapshotFile(FStringView File)
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlSnapshotFile>();
	Message->File = File;

	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::Pause()
{
	MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FTraceControlPause>(), ServiceAddress);
}

void FTraceControllerCommands::Resume()
{
	MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FTraceControlResume>(), ServiceAddress);
}

void FTraceControllerCommands::Bookmark(FStringView Label)
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlBookmark>();
	Message->Label = Label;
	
	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::Screenshot(FStringView Name, bool bShowUI)
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlScreenshot>();
	Message->Name = Name;
	Message->bShowUI = bShowUI;
	
	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::SetStatNamedEventsEnabled(bool bEnabled)
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlSetStatNamedEvents>();
	Message->bEnabled = bEnabled;

	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::SendStatusUpdateRequest()
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlStatusPing>();

	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::SendChannelUpdateRequest()
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlChannelsPing>();
	Message->KnownChannelCount = ChannelCount;

	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::SendSettingsUpdateRequest()
{
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlSettingsPing>();

	MessageEndpoint->Send(Message, ServiceAddress);
}

void FTraceControllerCommands::OnChannelsDesc(const FTraceControlChannelsDesc& Message)
{
	ChannelCount = Message.Channels.Num();
	if (ChannelCount != SettableChannels.Num())
	{
		for (int32 ChannelIdx = 0; ChannelIdx < ChannelCount; ++ChannelIdx)
		{
			uint32 Id = Message.Ids[ChannelIdx];
			// Do not add read-only channels to our list
			if (Message.ReadOnlyIds.Contains(Id))
			{
				continue;
			}
			const FString& ChannelName = Message.Channels[ChannelIdx];

			const uint64 Hash = HashName(FStringView(ChannelName));
			SettableChannels.FindOrAdd(Hash, Id);
		}
	}
}

void FTraceControllerCommands::SendSecure(FStringView Channels, bool bExcludeTail)
{
	//todo: If we had access to the target we could return here is secure connections
	//are not supported.
	const auto Message = FMessageEndpoint::MakeMessage<FTraceControlSecure>();
	Message->Channels = Channels;
	Message->bExcludeTail = bExcludeTail;
	MessageEndpoint->Send(Message, ServiceAddress);
}

