// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IMessageContext.h"
#include "ITraceControllerCommands.h"

class IMessageBus;
class FMessageEndpoint;
struct FTraceControlChannelsDesc;
class FTraceController;

/**
 * Logic for sending commands to a specific remote trace service.
 */
class FTraceControllerCommands : public ITraceControllerCommands
{
public:
	FTraceControllerCommands(const TSharedPtr<IMessageBus>& MessageBus, FMessageAddress Service, class FTraceController* Controller);
	virtual ~FTraceControllerCommands() override;

	void OnChannelsDesc(const FTraceControlChannelsDesc& Message);
	
private:	
	/* ITraceController interface */
	virtual void SetChannels(TConstArrayView<FStringView> ChannelsToEnable, TConstArrayView<FStringView> ChannelsToDisable) override;
	virtual void SetChannels(TConstArrayView<FString> ChannelsToEnable, TConstArrayView<FString> ChannelsToDisable) override;
	virtual void Send(FStringView Host, FStringView Channels, bool bExcludeTail) override;
	virtual void File(FStringView File, FStringView Channels, bool bExcludeTail, bool bTruncateFile) override;
	virtual void Stop() override;
	virtual void SnapshotSend(FStringView Host) override;
	virtual void SnapshotFile(FStringView File) override;
	virtual void Pause() override;
	virtual void Resume() override;
	virtual void Bookmark(FStringView Label) override;
	virtual void Screenshot(FStringView Name, bool bShowUI) override;
	virtual void SetStatNamedEventsEnabled(bool bEnabled) override;
	virtual void SendSecure(FStringView Channels, bool bExcludeTail) override;
	virtual void SendStatusUpdateRequest() override;
	virtual void SendChannelUpdateRequest() override;
	virtual void SendSettingsUpdateRequest() override;

private:
	/** Address of the remote trace service to control */
	FMessageAddress ServiceAddress;
	
	/** Our own endpoint for messages */
	TSharedPtr<FMessageEndpoint> MessageEndpoint;

	/** List of channels (hash of their name) that is possible to set (not read-only). */
	TMap<uint64, uint32> SettableChannels;

	/** The current known channel count. */
	int32 ChannelCount = 0;
};
