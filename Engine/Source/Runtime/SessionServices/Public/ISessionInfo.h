// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISessionInstanceInfo.h"

struct FSessionLogMessage;

/**
 * Interface for game instances.
 */
class ISessionInfo
{
public:

	/**
	 * Gets a read-only collection of all instances that belong to this session.
	 *
	 * @param OutInstances Will hold the collection of instances.
	 */
	virtual void GetInstances(TArray<TSharedPtr<ISessionInstanceInfo>>& OutInstances) const = 0;

	/**
	 * Gets the time at which the last update was received from this instance.
	 *
	 * @return The receive time.
	 */
	virtual const FDateTime& GetLastUpdateTime() const = 0;

	/**
	 * Gets the number of engine instances that are part of the session.
	 *
	 * @return The number of engine instances.
	 */
	virtual const int32 GetNumInstances() const = 0;

	/**
	 * Gets the session identifier.
	 *
	 * @return Session identifier.
	 */
	virtual const FGuid& GetSessionId() const = 0;

	/**
	 * Gets the name of the session.
	 *
	 * @return Session name.
	 */
	virtual const FString& GetSessionName() const = 0;

	/**
	 * Gets the name of the user that owns the session.
	 *
	 * @return User name.
	 */
	virtual const FString& GetSessionOwner() const = 0;

	/**
	 * Checks whether this is a standalone session.
	 *
	 * A session is standalone if has not been created from the Launcher.
	 *
	 * @return true if this is a standalone session, false otherwise.
	 */
	virtual const bool IsStandalone() const = 0;

	/**
	 * Terminates the session.
	 */
	virtual void Terminate() = 0;

public:

	/**
	 * A delegate that is executed when a new instance has been discovered.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_TwoParams(ISessionInfo, FInstanceDiscoveredEvent, const TSharedRef<ISessionInfo>& /*OwnerSession*/, const TSharedRef<ISessionInstanceInfo>& /*DiscoveredInstance*/)
	virtual FInstanceDiscoveredEvent& OnInstanceDiscovered() = 0;

	/**
	 * A delegate that is executed when a new log message has been received.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_ThreeParams(ISessionInfo, FLogReceivedEvent, const TSharedRef<ISessionInfo>& /*OwnerSession*/, const TSharedRef<ISessionInstanceInfo>& /*SessionInstance*/, const TSharedRef<FSessionLogMessage>& /*LogMessage*/);
	virtual FLogReceivedEvent& OnLogReceived() = 0;

public:

	/** Virtual destructor. */
	virtual ~ISessionInfo() = default;
};
