// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISessionInfo;
struct FSessionLogMessage;

/**
 * Interface for game instances.
 */
class ISessionInstanceInfo
{
public:

	/**
	 * Executes a console command on this engine instance.
	 *
	 * @param CommandString The command to execute.
	 */
	virtual void ExecuteCommand( const FString& CommandString ) = 0;

	/**
	 * Gets instance's build date.
	 *
	 * @return The build date string.
	 */
	virtual const FString& GetBuildDate() const = 0;

	/**
	 * Gets the name of the level that the instance is currently running.
	 *
	 * @return The name of the level.
	 */
	virtual const FString& GetCurrentLevel() const = 0;

	/**
	 * Gets the name of the device that this instance is running on.
	 *
	 * @return The device name string.
	 */
	virtual const FString& GetDeviceName() const = 0;

	/**
	 * Gets the instance's engine version number.
	 *
	 * @return Engine version.
	 */
	virtual int32 GetEngineVersion() const = 0;

	/**
	 * Gets the instance identifier.
	 *
	 * @return Instance identifier.
	 */
	virtual const FGuid& GetInstanceId() const = 0;

	/**
	 * Gets the name of this instance.
	 *
	 * @return The instance name string.
	 */
	virtual const FString& GetInstanceName() const = 0;

	/**
	 * Gets the instance type (i.e. Editor or Game).
	 *
	 * @return The game instance type string.
	 */
	virtual const FString& GetInstanceType() const = 0;

	/**
	 * Gets the time at which the last update was received from this instance.
	 *
	 * @return The receive time.
	 */
	virtual const FDateTime& GetLastUpdateTime() const = 0;

	/**
	 * Gets the collection of log entries received from this instance.
	 *
	 * @return Log entries.
	 */
	virtual const TArray<TSharedPtr<FSessionLogMessage>>& GetLog() = 0;

	/**
	 * Gets a reference to the session that owns this instance.
	 *
	 * @return Owner session.
	 */
	virtual TSharedPtr<ISessionInfo> GetOwnerSession() = 0;

	/**
	 * Gets the name of the platform that the instance is running on.
	 *
	 * @return Platform name string.
	 */
	virtual const FString& GetPlatformName() const = 0;

	/**
	 * Gets the instance's current game world time.
	 *
	 * @return World time in seconds.
	 */
	virtual float GetWorldTimeSeconds() const = 0;

	/**
	 * Check whether the current user is authorized to interact with this instance.
	 *
	 * @return true if the user is authorized, false otherwise.
	 */
	virtual bool IsAuthorized() const = 0;

	/**
	 * Checks whether this instance has already begun game play.
	 *
	 * @return true if game play has begun, false otherwise.
	 */
	virtual bool PlayHasBegun() const = 0;

	/** Terminates the instance. */
	virtual void Terminate() = 0;

public:

	/**
	 * Gets an event delegate that is executed when a new log message has been received.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_TwoParams(ISessionInstanceInfo, FLogReceivedEvent, const TSharedRef<ISessionInstanceInfo>&, const TSharedRef<FSessionLogMessage>&);
	virtual FLogReceivedEvent& OnLogReceived() = 0;

public:

	/** Virtual destructor. */
	virtual ~ISessionInstanceInfo() = default;
};
