// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISessionInstanceInfo.h"
#include "ISessionInfo.h"
#include "SessionLogMessage.h"

/**
 * Interface for the session manager.
 */
class ISessionManager
{
public:

	/**
	 * Adds an owner whose sessions we are interested in
	 *
	 * @param InOwner Session owner we want to view sessions from
	 */
	virtual void AddOwner( const FString& InOwner ) = 0;

	/**
	  * Gets the collection of currently selected engine instances.
	  *
	  * @return The selected instances.
	  */
	virtual const TArray<TSharedPtr<ISessionInstanceInfo>>& GetSelectedInstances() const = 0;

	/**
	 * Get the selected session - as chosen in the session browser
	 *
	 * @return The session ID selected
	 */
	virtual const TSharedPtr<ISessionInfo>& GetSelectedSession() const = 0;

	/**
	 * Gets the list of all discovered sessions.
	 *
	 * @param OutSessions Will hold the collection of sessions.
	 */
	virtual void GetSessions( TArray<TSharedPtr<ISessionInfo>>& OutSessions ) const = 0;


	/**
	 * Attempts to retrieve an instance by it's unique instance id.
	 * 
	 * @param Id Instance id
	 * @return Instance if it exist, otherwise a nullptr
	 */
	virtual TSharedPtr<ISessionInstanceInfo> GetInstance(const FGuid& Id) const = 0;

	/**
	 * Checks whether the given instance is currently selected.
	 *
	 * @return true if the instance is selected, false otherwise.
	 */
	virtual bool IsInstanceSelected( const TSharedRef<ISessionInstanceInfo>& Instance ) const = 0;

	/**
	 * Removes an owner whose sessions we are no longer interested in
	 *
	 * @param InOwner Session owner we want to remove
	 */
	virtual void RemoveOwner( const FString& InOwner ) = 0;

	/**
	 * Selects the specified session.
	 *
	 * @param Session The session to the select (can be NULL to select none).
	 * @return true if the session was selected, false otherwise.
	 */
	virtual bool SelectSession( const TSharedPtr<ISessionInfo>& Session ) = 0;

	/**
	 * Marks the specified item as selected or unselected.
	 *
	 * @param Instance The instance to mark.
	 * @param Selected Whether the instance should be selected (true) or unselected (false).
	 * @return true if the instance was selected, false otherwise.
	 */
	virtual bool SetInstanceSelected( const TSharedRef<ISessionInstanceInfo>& Instance, bool Selected ) = 0;

public:

	/**
	 * Returns a delegate that is executed before a session is being selected.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_TwoParams(ISessionManager, FCanSelectSessionEvent, const TSharedPtr<ISessionInfo>& /*Session*/, bool& /*OutCanSelect*/)
	virtual FCanSelectSessionEvent& OnCanSelectSession() = 0;

	/**
	 * Returns a delegate that is executed when an instance changes its selection state.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_TwoParams(ISessionManager, FInstanceSelectionChangedEvent, const TSharedPtr<ISessionInstanceInfo>& /*Instance*/, bool /*Selected*/)
	virtual FInstanceSelectionChangedEvent& OnInstanceSelectionChanged() = 0;

	/**
	 * Returns a delegate that is executed when the selected session received a log message from one of its instances.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_ThreeParams(ISessionManager, FLogReceivedEvent, const TSharedRef<ISessionInfo>& /*OwnerSession*/, const TSharedRef<ISessionInstanceInfo>& /*SessionInstance*/, const TSharedRef<FSessionLogMessage>& /*LogMessage*/);
	virtual FLogReceivedEvent& OnLogReceived() = 0;

	/**
	 * Returns a delegate that is executed when the selected session changed.
	 *
	 * @return The delegate.
	 */
	DECLARE_EVENT_OneParam(ISessionManager, FSelectedSessionChangedEvent, const TSharedPtr<ISessionInfo>& /*SelectedSession*/)
	virtual FSelectedSessionChangedEvent& OnSelectedSessionChanged() = 0;

	/**
	 * Returns a delegate that is executed when the list of sessions has changed.
	 *
	 * @return The delegate.
	 */
	virtual FSimpleMulticastDelegate& OnSessionsUpdated() = 0;

	/**
	 * Returns a delegate that is executed when a session instance is updated.
	 *
	 * @return The delegate.
	 */
	virtual FSimpleMulticastDelegate& OnSessionInstanceUpdated() = 0;

public:

	/** Virtual destructor. */
	virtual ~ISessionManager() = default;
};
