// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Interface for application session services.
 */
class ISessionService
{
public:

	/**
	 * Checks whether the service is running.
	 *
	 * @return true if the service is running, false otherwise.
	 * @see Start, Stop
	 */
	virtual bool IsRunning() = 0;

	/**
	 * Starts the service.
	 *
	 * @return true if the service was started, false otherwise.
	 * @see IsRunning, Stop
	 */
	virtual bool Start() = 0;

	/**
	 * Stops the service.
	 *
	 * @see IsRunning, Start
	 */
	virtual void Stop() = 0;

public:

	/** Virtual destructor. */
	virtual ~ISessionService() = default;
};
