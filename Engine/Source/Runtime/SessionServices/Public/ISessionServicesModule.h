// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "ISessionManager.h"
#include "ISessionService.h"

class ITraceController;

/**
 * Interface for session core modules.
 */
class ISessionServicesModule
	: public IModuleInterface
{
public:

	/**
	 * Gets the session manager.
	 *
	 * @return The session manager.
	 * @see GetSessionService
	 */
	virtual TSharedPtr<ISessionManager> GetSessionManager() = 0;

	/** 
	 * Gets the session service.
	 *
	 * @return The session service.
	 * @see GetSessionManager
	 */
	virtual TSharedPtr<ISessionService> GetSessionService() = 0;


	/**
	 * Get the trace controller
	 */
	virtual TSharedPtr<ITraceController> GetTraceController() = 0; 

public:

	/** Virtual destructor. */
	virtual ~ISessionServicesModule() = default;
};
