// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "ITraceControllerCommands.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Algo/Find.h"

/**
 * Describes the state of a single remote instance with Trace.
 */
struct FTraceStatus
{
	/**
	 * Update types
	 */
	enum class EUpdateType : uint8
	{
		Status			= 1 << 0,
		Settings		= 1 << 1,
		ChannelsDesc	= 1 << 2,
		ChannelsStatus	= 1 << 3,
		All				= Status|Settings|ChannelsDesc|ChannelsStatus
	};

	enum class ETraceSystemStatus : uint8
	{
		NotAvailable,
		Available,
		TracingToServer,
		TracingToFile,
		TracingToSecureNetwork,
		TracingToCustomRelay,

		NumValues,
	};

	struct FChannelPreset
	{
		FChannelPreset(const FString& InName, const FString& InChannels, bool bInIsReadOnly)
			: Name(InName)
			, ChannelList(InChannels)
			, bIsReadOnly(bInIsReadOnly)
		{
		}

		FString Name;
		FString ChannelList;
		bool bIsReadOnly;
	};

	struct FCapability
	{
		FString Name;
	};
	
	struct FSettings
	{
		/** If worker thread is used or TraceLog is pumped on end frame. */
		bool bUseWorkerThread;
		/** If important cache is enabled */
		bool bUseImportantCache;
		/** Size of tail buffer */
		uint32 TailSizeBytes;
		/** The channel presets defined by the process. */
		TArray<FChannelPreset> ChannelPresets;
		/** Capabilities of the target */
		TArray<FCapability> Capabilities;
	};

	struct FChannel
	{
		/** Channel name */
		FString Name;
		/** Description string */
		FString Description;
		/** Remote system id. This may be unique per machine. */
		uint32 Id;
		/** If channel is currently enabled */
		bool bEnabled;
		/** If it's possible to toggle channel. Read only channels
		 * must be set on process start using command line arguments. */
		bool bReadOnly;
	};

	struct FStats
	{
		/** Number of bytes sent to server or file. */
		uint64 BytesSent;
		/** Number of (uncompressed) bytes traced from process. */
		uint64 BytesTraced;
		/** Total memory used by TraceLog */
		uint64 MemoryUsed;
		/** Allocated memory for important events */
		uint32 CacheAllocated;
		/** Memory used for important events */
		uint32 CacheUsed;
		/** Wasted space for important events cache */
		uint32 CacheWaste;
	};
	
	/** Session id of the process we're controlling */
	FGuid SessionId;
	/** Instance id of the process we're controlling */
	FGuid InstanceId;
	/** If tracing is active */
	bool bIsTracing = false;
	/** If tracing is paused */
	bool bIsPaused = false;
	/** If stats are emitted as named events */
	bool bAreStatNamedEventsEnabled = false;
	/** Endpoint of active trace */
	FString Endpoint;
	/** Session identifier for the trace. */
	FGuid SessionGuid;
	/** Unique identifier for the trace */
	FGuid TraceGuid;
	/** Settings */
	FSettings Settings;
	/** State of channels */
	TMap<uint32, FChannel> Channels;
	/** Stats */
	FStats Stats;
	/** Timestamp when the state of the trace was captured */
	FDateTime StatusTimestamp;
	/** The status of the trace system. */
	ETraceSystemStatus TraceSystemStatus;

	bool HasChannel(FStringView ChannelName) const
	{
		for (const auto& ChannelPair : Channels)
		{
			FStringView Name (ChannelPair.Value.Name);
			if (Name.Compare(ChannelName, ESearchCase::IgnoreCase) == 0)
			{
				return true;
			}
		}
		return false;
	}
};

ENUM_CLASS_FLAGS(FTraceStatus::EUpdateType);

struct FSecureConnection
{
	FString Host;
	uint16 Port;
	TArray<uint8> AuthorizationToken;
	TArray<uint8> CertificateAuthority;
	FString Identity;
};

/**
 * Interface for monitoring known trace service instances. ITraceController is a singleton which tracks
 * all known instances and provides callback for status updates and interfaces to issue commands to one
 * or more remote instances.
 *
 * Example usage:
 * ```
 * // I've found the session instance that we want to control
 * FGuid InstanceId = ...;
 * // Get the controller
 * TSharedPtr<ITraceController> TraceController = SessionServicesModule.GetTraceController();
 * 
 * // To control just a single instance use the session instance id.
 * TraceController->WithInstance(InstanceId, [](const FTraceStatus& Status, ITraceControllerCommands& Commands){
 *		Commands.Send("localhost", "audio,audiomixer,bookmark,log");
 *		Commands.Bookmark("My remote bookmark");
 * });
 * ```
 */
class ITraceController
{
public:
	
	virtual ~ITraceController() = default;

	/**
	 * Request to discover a specific instance
	 * @param SessionId Session id of the instance
	 * @param InstanceId Instance id
	 */
	virtual void SendDiscoveryRequest(const FGuid& SessionId, const FGuid& InstanceId) const = 0;

	/**
	 * Request to discover all instances
	 */
	virtual void SendDiscoveryRequest() = 0;

	/**
	 * Request update of the status from all sessions and instances.
	 */
	// UE_DEPRECATED(5.8, "Use the version from ITraceControllerCommands as this one can cause high traffic when used used in large networks.")
	virtual void SendStatusUpdateRequest() = 0;

	/**
	 * Request update of channel status from all sessions and instances.
	 */
	// UE_DEPRECATED(5.8, "Use the version from ITraceControllerCommands as this one can cause high traffic when used used in large networks.")
	virtual void SendChannelUpdateRequest() = 0;

	/**
	 * Request update of setting state from all sessions and instances.
	 */
	// UE_DEPRECATED(5.8, "Use the version from ITraceControllerCommands as this one can cause high traffic when used used in large networks.")
	virtual void SendSettingsUpdateRequest() = 0;

	/**
	 * Generic event for updates of status
	 */
	DECLARE_EVENT_ThreeParams(ITraceController, FStatusReceivedEvent, const FTraceStatus&, FTraceStatus::EUpdateType, ITraceControllerCommands&);

	/**
	 * Event triggered whenever status is updated for any known session. 
	 * A reference to the status and what has changed is provided along with a 
	 * structure to issue commands.
	 */
	virtual FStatusReceivedEvent& OnStatusReceived() = 0;


	DECLARE_EVENT_TwoParams(ITraceController, FSecureResponseEvent, const FTraceStatus&, const FSecureConnection&)

	/**
	 * Event triggered when a response to a secure connection request has been made.
	 */
	virtual FSecureResponseEvent& OnSecureResponse() = 0;

	/**
	 * Return true if an instance with the provided Id exists and has been discovered.
	 */
	virtual bool HasAvailableInstance(const FGuid& InstanceId) = 0;

	/**
	 * Returns true if the last message received for a specific instance is newer than TimeoutSec.
	 * @param InstanceId The id of the instance to check.
	 * @param TimeoutSec The maximum number of seconds that must have passed since the last message for it to be considered active.
	 */
	virtual bool IsActive(FGuid InstanceId, double TimeoutSec = 5.0f) const = 0;

	typedef TFunction<void(const FTraceStatus&, ITraceControllerCommands&)> FCallback;

	/**
	 * Execute a function on a specific session.
	 * @param InstanceId InstanceId to issue command to
	 * @param Func Functor to execute
	 */
	virtual void WithInstance(FGuid InstanceId, FCallback Func) = 0;


	/**
	 * Queries a session about a specific capability.
	 * @param InstanceId InstanceId to query
	 * @param Name Name of capability
	 * @return True if session lists capability, false otherwise
	 */
	virtual bool HasCapability(FGuid InstanceId, FStringView Name) = 0;

	/**
	 * Create an instance using a custom message bus
	 * @param MessageBus Message bus the controller will use
	 */
	SESSIONSERVICES_API static TSharedPtr<ITraceController> Create(TSharedPtr<class IMessageBus>& MessageBus);
};

