// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"

/**
 * Interface to send commands to a single remote trace service.
 */
class ITraceControllerCommands
{
public:

	virtual ~ITraceControllerCommands() = default;
	
	/**
	 * Enables or disables channels by name
	 * @param ChannelsToEnable  List of channels to enable
	 * @param ChannelsToDisable List of channels to disable
	 */
	virtual void SetChannels(TConstArrayView<FStringView> ChannelsToEnable, TConstArrayView<FStringView> ChannelsToDisable) = 0;

	/**
	 * Enables or disables channels by name
	 * @param ChannelsToEnable  List of channels to enable
	 * @param ChannelsToDisable List of channels to disable
	 */
	virtual void SetChannels(TConstArrayView<FString> ChannelsToEnable, TConstArrayView<FString> ChannelsToDisable) = 0;
	
	/**
	 * Start a trace on selected instances to the provided host, using a set of channels.
	 * @param Host Host to send the trace to
	 * @param Channels Comma separated list of channels to enable
	 * @param bExcludeTail If the tail (circular buffer of recent events) should be included
	 */
	virtual void Send(FStringView Host, FStringView Channels, bool bExcludeTail = false) = 0;

	/**
	 * Start a trace on selected instances to a file on the instance, using a set of channels.
	 * @param File Path on the instance. ".utrace" will be appended
	 * @param Channels Comma separated list of channels to enable
	 * @param bExcludeTail If the tail (circular buffer of recent events) should be included
	 * @param bTruncateFile If the file should be truncated (if already exists)
	 */
	virtual void File(FStringView File, FStringView Channels, bool bExcludeTail = false, bool bTruncateFile = false) = 0;


	/**
	 * Start a trace on selected instance using a secure connection. The initiator is required
	 * to connect to the port using the options specified in the response.
	 * @param Channels Comma separated list of channels to enable
	 * @param bExcludeTail If the tail (circular buffer of recent events) should be included
	 */
	virtual void SendSecure(FStringView Channels, bool bExcludeTail = false) = 0;

	/**
	 * On selected instances, make a snapshot of the tail (circular buffer of recent events)
	 * and send to the provided host.
	 * @param Host Host to send the trace to
	 */
	virtual void SnapshotSend(FStringView Host) = 0;
	
	/**
	 * On selected instances, make a snapshot of the tail (circular buffer of recent events)
	 * and save to a file.
	 * @param File Path on the instance. ".utrace" will be appended
	 */
	virtual void SnapshotFile(FStringView File) = 0;

	/**
	 * Pause tracing by muting all (non-readonly) channels.
	 */
	virtual void Pause() = 0;

	/**
	 * Resume tracing (from paused) by enabling the previously enabled channels.
	 */
	virtual void Resume() = 0;

	/**
	 * Stop active trace.
	 */
	virtual void Stop() = 0;

	/**
	 * Insert bookmark into the trace.
	 * @param Label Label of bookmark
	 */
	virtual void Bookmark(FStringView Label) = 0;

	/**
	 * Insert screenshot into the trace.
	 * @param Name Name of the screenshot
	 * @param bShowUI If the UI should be visible in the image
	 */
	virtual void Screenshot(FStringView Name, bool bShowUI) = 0;

	/**
	 * Set the StatNamedEvents flag.
	 * @param bEnabled The value to assign to the StatNamedEvents flag.
	 */
	virtual void SetStatNamedEventsEnabled(bool bEnabled) = 0;

	/**
	 * Request update of the status.
	 */
	virtual void SendStatusUpdateRequest() = 0;

	/**
	 * Request update of channel status.
	 */
	virtual void SendChannelUpdateRequest() = 0;

	/**
	 * Request update of setting state.
	*/
	virtual void SendSettingsUpdateRequest() = 0;
};

