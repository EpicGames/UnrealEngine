// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

/**
 * Structure for log messages.
 */
struct FSessionLogMessage
{
	/** Holds the log category. */
	FName Category;

	/** Holds the identifier of the engine instance that generated this log message. */
	FGuid InstanceId;

	/** Holds the name of the engine instance that generated this log message. */
	FString InstanceName;

	/** Holds the message text. */
	FString Text;

	/** Holds the time at which the message was generated. */
	FDateTime Time;

	/** Holds the number of seconds from the start of the instance at which the message was generated. */
	double TimeSeconds;

	/** Holds the verbosity type. */
	ELogVerbosity::Type Verbosity;

public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InInstanceId The identifier of the instance that generated the log message.
	 * @param InInstanceName The name of the engine instance that generated the log message.
	 * @param InTimeSeconds The number of seconds from the start of the instance at which the message was generated.
	 * @param InText The message text.
	 * @param InVerbosity The verbosity type.
	 * @param InCategory The log category.
	 */
	FSessionLogMessage(const FGuid& InInstanceId, const FString& InInstanceName, float InTimeSeconds, const FString& InText, ELogVerbosity::Type InVerbosity, const FName& InCategory)
		: Category(InCategory)
		, InstanceId(InInstanceId)
		, InstanceName(InInstanceName)
		, Text((InCategory != NAME_None) ? InCategory.ToString() + TEXT(": ") + InText : InText)
		, Time(FDateTime::Now())
		, TimeSeconds(InTimeSeconds)
		, Verbosity(InVerbosity)
	{ }

public:

	/**
	 * Implements a predicate to compare two log messages by log time.
	 */
	struct TimeComparer
	{
		bool operator()(const TSharedPtr<FSessionLogMessage>& A, const TSharedPtr<FSessionLogMessage>& B) const
		{
			return A->Time < B->Time;
		}
	};
};
