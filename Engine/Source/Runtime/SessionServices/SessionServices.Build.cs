// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SessionServices : ModuleRules
{
	public SessionServices(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"EngineMessages",
				"SessionMessages",
			});

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Messaging",
				"MessagingCommon",
			});
	}
}
